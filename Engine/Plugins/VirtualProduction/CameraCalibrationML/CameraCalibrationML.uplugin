{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Camera Calibration Machine Learning",
	"Description": "Reference implementation of a machine learning approach to distortion calibration.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Hidden": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CameraCalibrationML",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CameraCalibration",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		}
	]
}